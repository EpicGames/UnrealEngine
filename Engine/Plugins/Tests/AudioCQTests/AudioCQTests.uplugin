{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "Audio Code Quality Tests",
  "Description": "Audio Code Quality Tests",
  "Category": "Testing",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "DocsURL": "",
  "MarketplaceURL": "",
  "SupportURL": "",
  "CanContainContent": false,
  "IsBetaVersion": false,
  "Installed": false,
  "Modules": [
    {
      "Name": "AudioCQTests",
      "Type": "DeveloperTool",
      "LoadingPhase": "Default"
    }
  ]
}
