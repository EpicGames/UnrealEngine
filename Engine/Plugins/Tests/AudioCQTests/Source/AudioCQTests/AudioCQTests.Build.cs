// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using UnrealBuildTool.Rules;

public class AudioCQTests : ModuleRules
{
	public AudioCQTests(ReadOnlyTargetRules Target)
		: base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
					"AudioExtensions",
					"AudioLinkEngine",
					"AudioMixer",
					"Core",
					"CoreUObject",
					"CQTest",
					"Engine",
					"SignalProcessing",
					"SoundFieldRendering",
				 }
			);
	}
}
