// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Algo/RandomShuffle.h"
#include "AudioRenderSchedulerTester.h"
#include "CoreMinimal.h"
#include "CQTest.h"

TEST_CLASS_WITH_TAGS(AudioRenderSchedulerTests, "Audio.RenderScheduler", "[MustPass]")
{
	TUniquePtr<IAudioRenderSchedulerTester> Tester;
	TArray<int> Results;

	void AddStepRange(int Num)
	{
		// Add the steps in random order for better test coverage
		TArray<int> Range;
		for (int i = 0; i < Num; ++i)
		{
			Range.Add(i);
		}

		Algo::RandomShuffle(Range);
		for (int i : Range)
		{
			Tester->AddStep(i);
		}
	}

	BEFORE_EACH()
	{
		Tester = IAudioRenderSchedulerTester::Create();
	}

	AFTER_EACH()
	{
		Tester.Reset();
		Results.Empty();
	}

	TEST_METHOD(EmptyTest)
	{
		TestCommandBuilder.Do([this]()
			{
				Results = Tester->Run();
				ASSERT_THAT(AreEqual(Results.Num(), 0));
			});
	}

	TEST_METHOD(OneStep)
	{
		TestCommandBuilder.Do([this]()
			{
				Tester->AddStep(7);

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 1));
				ASSERT_THAT(AreEqual(Results[0], 7));
			});
	}

	TEST_METHOD(TwoSteps)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(2);
				Tester->AddDependency(0, 1);

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 2));
				ASSERT_THAT(AreEqual(Results[0], 0));
				ASSERT_THAT(AreEqual(Results[1], 1));
			});
	}

	TEST_METHOD(UnconnectedSteps)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(10);

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 10));
				for (int i = 0; i < 10; ++i)
				{
					ASSERT_THAT(IsTrue(Results.Contains(i)));
				}
			});
	}

	TEST_METHOD(SinglePrereq)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(10);
				for (int i = 1; i < 10; ++i)
				{
					Tester->AddDependency(0, i);
				}

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 10));
				for (int i = 0; i < 10; ++i)
				{
					ASSERT_THAT(IsTrue(Results.Contains(i)));
				}
				ASSERT_THAT(AreEqual(Results[0], 0));
			});
	}

	TEST_METHOD(SinglePrereqDoubled)
	{
		TestCommandBuilder.Do([this]()
			{
				// Make sure that multiple redundant dependencies work just as well as single dependencies
				AddStepRange(10);
				for (int i = 1; i < 10; ++i)
				{
					Tester->AddDependency(0, i);
					Tester->AddDependency(0, i);
				}

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 10));
				for (int i = 0; i < 10; ++i)
				{
					ASSERT_THAT(IsTrue(Results.Contains(i)));
				}
				ASSERT_THAT(AreEqual(Results[0], 0));
			});
	}

	TEST_METHOD(SinglePostreq)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(10);
				for (int i = 1; i < 10; ++i)
				{
					Tester->AddDependency(i, 0);
				}

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 10));
				for (int i = 0; i < 10; ++i)
				{
					ASSERT_THAT(IsTrue(Results.Contains(i)));
				}
				ASSERT_THAT(AreEqual(Results[9], 0));
			});
	}

	TEST_METHOD(BasicCylce)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(3);
				Tester->AddDependency(0, 1);
				Tester->AddDependency(1, 2);
				Tester->AddDependency(2, 0);

				Results = Tester->Run();

				// One link of the cycle should be broken, but we don't know exactly where.
				// Results should be some cyclic permutation of [0, 1, 2].
				ASSERT_THAT(AreEqual(Results.Num(), 3));
				ASSERT_THAT(AreEqual(Results[1], (Results[0] + 1) % 3));
				ASSERT_THAT(AreEqual(Results[2], (Results[1] + 1) % 3));
			});
	}

	TEST_METHOD(MiddleCycle)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(5);
				Tester->AddDependency(0, 1);
				Tester->AddDependency(1, 2);
				Tester->AddDependency(2, 3);
				Tester->AddDependency(3, 1);
				Tester->AddDependency(1, 4);
				Tester->AddDependency(2, 4);
				Tester->AddDependency(3, 4);

				Results = Tester->Run();

				// Should be 0, then a cyclic permutation of [1,2,3], then 4.
				ASSERT_THAT(AreEqual(Results.Num(), 5));
				ASSERT_THAT(AreEqual(Results[0], 0));
				ASSERT_THAT(AreEqual(Results[4], 4));
				ASSERT_THAT(IsTrue(Results.Contains(1) && Results.Contains(2) && Results.Contains(3)));
				ASSERT_THAT(AreEqual((Results[2] - Results[1] - 1) % 3, 0));
				ASSERT_THAT(AreEqual((Results[3] - Results[2] - 1) % 3, 0));
			});
	}

	TEST_METHOD(TwoCycles)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(7);
				Tester->AddDependency(0, 1);
				Tester->AddDependency(1, 2);
				Tester->AddDependency(2, 0);

				Tester->AddDependency(0, 3);
				Tester->AddDependency(1, 3);
				Tester->AddDependency(2, 3);

				Tester->AddDependency(3, 4);
				Tester->AddDependency(3, 5);
				Tester->AddDependency(3, 6);

				Tester->AddDependency(4, 5);
				Tester->AddDependency(5, 6);
				Tester->AddDependency(6, 4);

				Results = Tester->Run();

				// Should be a cyclic permutation of [0,1,2], then 3, then a cyclic permutation of [4,5,6].
				ASSERT_THAT(AreEqual(Results.Num(), 7));
				ASSERT_THAT(AreEqual((Results[1] - Results[0] - 1) % 3, 0));
				ASSERT_THAT(AreEqual((Results[2] - Results[1] - 1) % 3, 0));
				ASSERT_THAT(AreEqual(Results[3], 3));
				ASSERT_THAT(AreEqual((Results[5] - Results[4] - 1) % 3, 0));
				ASSERT_THAT(AreEqual((Results[6] - Results[5] - 1) % 3, 0));
			});
	}

	TEST_METHOD(OnlyPhantomSteps)
	{
		TestCommandBuilder.Do([this]()
			{
				// In this test we register a dependency, but not the actual steps involved.
				Tester->AddDependency(0, 1);

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 0));
			});
	}

	TEST_METHOD(MiddlePhantomSteps)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(2);
				Tester->AddDependency(1, 2);
				Tester->AddDependency(2, 3);
				Tester->AddDependency(3, 0);

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 2));
				ASSERT_THAT(AreEqual(Results[0], 1));
				ASSERT_THAT(AreEqual(Results[1], 0));
			});
	}

	TEST_METHOD(DenseGraph)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(10);
				for (int i = 0; i < 10; ++i)
				{
					for (int j = i; j < 10; ++j)
					{
						Tester->AddDependency(i, j);
					}
				}

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 10));
				for (int i = 0; i < 10; ++i)
				{
					ASSERT_THAT(AreEqual(Results[i], i));
				}
			});
	}

	TEST_METHOD(DenseGraphWithCycles)
	{
		TestCommandBuilder.Do([this]()
			{
				AddStepRange(10);
				for (int i = 0; i < 10; ++i)
				{
					for (int j = 0; j < 10; ++j)
					{
						Tester->AddDependency(i, j);
					}
				}

				Results = Tester->Run();

				// The order is completely arbitrary here, but we check that the scheduler managed to chew through all the cycles
				//  and schedule every step.
				ASSERT_THAT(AreEqual(Results.Num(), 10));
				for (int i = 0; i < 10; ++i)
				{
					ASSERT_THAT(IsTrue(Results.Contains(i)));
				}
			});
	}

	TEST_METHOD(TwoCyclesRepeated)
	{
		TestCommandBuilder.Do([this]()
			{
				// 0-1-2 form a cycle, then 3, then 4-5-6 in a cycle
				AddStepRange(7);
				Tester->AddDependency(0, 1);
				Tester->AddDependency(1, 2);
				Tester->AddDependency(2, 0);

				Tester->AddDependency(0, 3);
				Tester->AddDependency(1, 3);
				Tester->AddDependency(2, 3);

				Tester->AddDependency(3, 4);
				Tester->AddDependency(3, 5);
				Tester->AddDependency(3, 6);

				Tester->AddDependency(4, 5);
				Tester->AddDependency(5, 6);
				Tester->AddDependency(6, 4);

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 7));
				ASSERT_THAT(AreEqual((Results[1] - Results[0] - 1) % 3, 0));
				ASSERT_THAT(AreEqual((Results[2] - Results[1] - 1) % 3, 0));
				ASSERT_THAT(AreEqual(Results[3], 3));
				ASSERT_THAT(AreEqual((Results[5] - Results[4] - 1) % 3, 0));
				ASSERT_THAT(AreEqual((Results[6] - Results[5] - 1) % 3, 0));

				// Check that running without changing anything gets the same results
				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 7));
				ASSERT_THAT(AreEqual((Results[1] - Results[0] - 1) % 3, 0));
				ASSERT_THAT(AreEqual((Results[2] - Results[1] - 1) % 3, 0));
				ASSERT_THAT(AreEqual(Results[3], 3));
				ASSERT_THAT(AreEqual((Results[5] - Results[4] - 1) % 3, 0));
				ASSERT_THAT(AreEqual((Results[6] - Results[5] - 1) % 3, 0));

				// Now rearrange a bit and check that the results update correctly.
				Tester->RemoveDependency(3, 4);
				Tester->RemoveDependency(4, 5);
				Tester->RemoveDependency(6, 4);
				Tester->AddDependency(4, 0);
				Tester->AddDependency(4, 1);
				Tester->AddDependency(4, 2);

				Results = Tester->Run();

				ASSERT_THAT(AreEqual(Results.Num(), 7));
				ASSERT_THAT(AreEqual(Results[0], 4));
				ASSERT_THAT(AreEqual((Results[2] - Results[1] - 1) % 3, 0));
				ASSERT_THAT(AreEqual((Results[3] - Results[2] - 1) % 3, 0));
				ASSERT_THAT(AreEqual(Results[4], 3));
				ASSERT_THAT(AreEqual(Results[5], 5));
				ASSERT_THAT(AreEqual(Results[6], 6));
			});
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS