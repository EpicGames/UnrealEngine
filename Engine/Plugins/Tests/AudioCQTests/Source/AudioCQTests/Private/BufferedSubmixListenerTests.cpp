// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "AudioDevice.h"
#include "Misc/AutomationTest.h"
#include "Components/AudioComponent.h"
#include "CoreMinimal.h"
#include "CQTest.h"
#include "DownmixedBufferedSubmixListener.h"
#include "Engine/World.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformProcess.h"
#include "HAL/ThreadSafeBool.h"
#include "Math/UnrealMathUtility.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundSubmix.h"
#include "Tasks/Pipe.h"
#include "TestSineSynthComponent.h"

// This class creates a task that polls every 10ms and asks the BufferedSubmixListener for more data
// then analyzes it to make sure it is the intended sine wave
class FCQTestPoller
{
public:
	FCQTestPoller(FAudioDevice* InAudioDevice, USoundSubmix* InSubmix, int32 InFrameCount, int32 InChannelCount, int32 InSampleRate)
		: Pipe(TEXT("CQTestPoller")) 
		, AudioDevice(InAudioDevice)
		, Submix(InSubmix)
		, FrameCount(InFrameCount)
		, ChannelCount(InChannelCount)
		, SampleRate(InSampleRate)
	{
		const FString VoiceChatSimName(TEXT("VoiceChatSimTest"));
		Listener = MakeShared<FDownmixedBufferedSubmixListener>(InChannelCount, InSampleRate, &VoiceChatSimName);
		Listener->OnSubmixBufferWritten.BindRaw(this, &FCQTestPoller::HandleOnSubmixBufferWritten);
		TestBuffer.Reserve(InFrameCount * InChannelCount);
	}

	// Start polling loop in a task
	void Start()
	{
		bStop = false;

		if (AudioDevice && Listener.IsValid())
		{
			Listener->Start(AudioDevice, Submix);
		}

		// Launch a "thread-like" sequential task in the Pipe.
		PollTask = Pipe.Launch(TEXT("CQTestPoller::Run"), [this]()
			{
				while (!bStop)
				{
					// Sleep ~10ms to simulate input audio driver task scheduling
					FPlatformProcess::Sleep(0.01f); // 10ms
					PollListenerQueue();
				}
			});
	}

	// Signal stop and wait for the task and pipe to quiesce
	void Stop()
	{
		bStop = true;

		if (Listener.IsValid())
		{
			if (AudioDevice)
			{
				Listener->Stop(AudioDevice);
			}

			Listener->OnSubmixBufferWritten.Unbind();
			Listener.Reset();
		}

		AudioDevice = nullptr;

		PollTask.Wait();
		Pipe.WaitUntilEmpty(); // ensure pipe has no more work
	}

	// This function is called back by the BufferedSubmixListener when it has received a buffer
	// Here we are just checking that there is actual signal coming in so we can start the test
	void HandleOnSubmixBufferWritten(const float* SampleBuffer, int32 InFrameCount, int32 InNumChannels)
	{
		BuffersReceived++;

		TArrayView<const float>ViewBuffer{ SampleBuffer, InFrameCount * InNumChannels };

		// Compute a simple RMS without regard for channels, just to make sure there is a signal present
		double SumSquares = 0.0;
		for (int32 Sample = 0; Sample < InFrameCount * InNumChannels; Sample++)
		{
			const double SampleValue = static_cast<double>(SampleBuffer[Sample]);
			SumSquares += SampleValue * SampleValue;
		}
		const double MeanSquare = (InFrameCount * InNumChannels > 0) ? (SumSquares / static_cast<double>(InFrameCount * InNumChannels)) : 0.0;
		const float Rms = static_cast<float>(FMath::Sqrt(MeanSquare));

		LatestRMS.store(Rms);
	}

	// Change the output format of the listener and update the poller's local format to match
	void SetOutputFormat(int32 InChannelCount, int32 InSampleRate)
	{
		if (Listener.IsValid())
		{
			Listener->SetOutputFormat(InChannelCount, InSampleRate);
		}

		const int32 NewFrameCount = InSampleRate / 100;
		FrameCount.store(NewFrameCount);
		ChannelCount.store(InChannelCount);
		SampleRate.store(InSampleRate);
		TestBuffer.SetNumUninitialized(NewFrameCount * InChannelCount);
	}

	// Functions to access atomic values during the test process
	void SetTestSineFrequency(float ExpectedFreq) { ExpectedSineFreq = ExpectedFreq; }
	int32 GetBuffersReceived() { return BuffersReceived.load(); }
	float GetLatestRMS() { return LatestRMS.load(); }
	int32 GetSuccessfulTestCount() { return SuccessfulTests.load(); }
	int32 GetFailedTestCount() { return FailedTests.load(); }
	void ClearSetupValues() { BuffersReceived.store(0); LatestRMS.store(0.0f); }
	void ClearTestValues() { SuccessfulTests.store(0); FailedTests.store(0); }

private:
	// This function is called on a 10ms callback to simulate VoiceChat timing
	// This fetches a 10ms buffer of data from the BufferedSubmixListener and checks if it is actually a sine wave
	void PollListenerQueue()
	{
		const int32 CurrentFrameCount = FrameCount.load();
		const int32 CurrentChannelCount = ChannelCount.load();
		const int32 CurrentSampleRate = SampleRate.load();
		const int32 RequiredSamples = CurrentFrameCount * CurrentChannelCount;

		if (Listener.IsValid() && Listener->GetNumAvailableSamples() >= RequiredSamples)
		{
			TestBuffer.SetNumUninitialized(RequiredSamples);
			float* FilledBuffer = TestBuffer.GetData();
			int32 SamplesPopped = 0;
			Listener->GetBuffer(FilledBuffer, RequiredSamples, SamplesPopped);

			bool TestPasses = CurrentChannelCount > 0;

			for (int32 Channel = 0; Channel < CurrentChannelCount; Channel++)
			{
				float Amplitude = 0.0f;
				float Phase = 0.0f;
				TestPasses &= IsSineAtFrequency(FilledBuffer, SamplesPopped / CurrentChannelCount, CurrentSampleRate, CurrentChannelCount, Channel, ExpectedSineFreq, Amplitude, Phase);
			}

			if (TestPasses)
			{
				SuccessfulTests++;
			}
			else
			{
				FailedTests++;
			}
		}
	}

	// Returns true if buffer is well-fit by a sine at ExpectedFreq within tolerances.
	// Also outputs estimated amplitude and phase.
	bool IsSineAtFrequency(const float* Buffer, int32 InNumFrames, float InSampleRate, int32 InNumChannels, int32 ChannelIdx,
		float ExpectedFreq, float& OutAmplitude, float& OutPhase, float RMSErrorTolerance = 0.005f)
	{
		if (!Buffer || InNumFrames <= 0 || InSampleRate <= 0.f || ExpectedFreq <= 0.f || InNumChannels <= 0 || ChannelIdx >= InNumChannels)
		{
			return false;
		}

		const float TwoPi = UE_TWO_PI;
		const float Omega = TwoPi * ExpectedFreq / InSampleRate;

		// Correlate against cos(omega n) and sin(omega n)
		double SumCos = 0.0;
		double SumSin = 0.0;
		double Energy = 0.0;

		for (int32 Sample = 0; Sample < InNumFrames; ++Sample)
		{
			const float SampleVal = Buffer[Sample * InNumChannels + ChannelIdx];
			Energy += SampleVal * SampleVal;

			const float CosVal = FMath::Cos(Omega * Sample);
			const float SinVal = FMath::Sin(Omega * Sample);

			SumCos += SampleVal * CosVal;
			SumSin += SampleVal * SinVal;
		}

		// Scale to get best-fit sinusoid coefficients for x[n] ~ A * sin(omega n + phi)
		// A*cos(phi) ~ (2/N) SumSin ; A*sin(phi) ~ (2/N) SumCos (depending on convention).
		// Here derive A and phi consistently:
		const double Norm = 2.0 / double(InNumFrames);
		const double A_cosphi = Norm * SumSin; // correlates with sin
		const double A_sinphi = Norm * SumCos; // correlates with cos

		OutAmplitude = float(FMath::Sqrt(A_cosphi * A_cosphi + A_sinphi * A_sinphi));
		OutPhase = (UE_PI / 2.0) - float(FMath::Atan2(A_cosphi, A_sinphi)); // so that sin(n*omega + phi) fits

		// No point in testing energy if there is no amplitude
		if (OutAmplitude < RMSErrorTolerance)
		{
			return false;
		}

		// Compute reconstruction and RMSE
		double SumSquaredError = 0.0;
		for (int32 Sample = 0; Sample < InNumFrames; ++Sample)
		{
			const float SineVal = OutAmplitude * FMath::Sin(Omega * Sample + OutPhase);
			const double Error = double(Buffer[Sample * InNumChannels + ChannelIdx]) - double(SineVal);
			SumSquaredError += Error * Error;
		}

		const double RMSError = FMath::Sqrt(SumSquaredError / double(InNumFrames));

		// Optional sanity checks on amplitude bounds if you expect unit amplitude, etc.
		// Return pass if RMSE small enough
		return RMSError <= RMSErrorTolerance;
	}


private:
	// Poll task variables
	UE::Tasks::FPipe Pipe;
	UE::Tasks::FTask PollTask;
	FThreadSafeBool bStop{ false };

	// Variables needed to connect to the listener and parameters to format the data as desired
	FAudioDevice* AudioDevice = nullptr;
	USoundSubmix* Submix;
	std::atomic<int32> FrameCount;
	std::atomic<int32> ChannelCount;
	std::atomic<int32> SampleRate;

	// Device under test
	TSharedPtr<FDownmixedBufferedSubmixListener> Listener;

	// Test variables
	float ExpectedSineFreq;
	std::atomic<int32> BuffersReceived{ 0 };
	std::atomic<float> LatestRMS{ 0.0f };
	std::atomic<int32> SuccessfulTests{ 0 };
	std::atomic<int32> FailedTests{ 0 };
	TArray<float> TestBuffer;
};

TEST_CLASS_WITH_FLAGS(BufferedSubmixListener_SmokeTest, "Audio.Mixer", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	// Test constants
	static constexpr uint32 SampleRate = 48000;
	static constexpr float  FrequencyHz = 440.0f;
	static constexpr float  TargetSendLevel = 1.0f;
	static constexpr float  MinDetectableRMSForSignal = 0.0001f; 

	static constexpr int32 FrameCount = SampleRate / 100; // VoiceChat wants data in 10ms chunks
	static constexpr int32 ChannelCount = 2; // Proximity VoiceChat wants a stereo stream

	// Owned objects
	TStrongObjectPtr<USoundSubmix> TestSubmix = nullptr;
	TStrongObjectPtr<UTestSineSynthComponent> SineGenerator = nullptr;
	TStrongObjectPtr<UTestSineSynthComponent> InterferenceTone = nullptr;

	FAudioDevice* AudioDevice = nullptr;

	// Test rig
	TUniquePtr<FCQTestPoller> VoiceChatSim;
	int32 FirstTestBuffer = 0;

	BEFORE_EACH()
	{
		// 1) Get main audio device
		AudioDevice = GEngine ? GEngine->GetMainAudioDeviceRaw() : nullptr;
		ASSERT_THAT(IsNotNull(AudioDevice));

		// 2) Create a transient submix instance for routing
		TestSubmix = TStrongObjectPtr<USoundSubmix>(NewObject<USoundSubmix>());
		if (AudioDevice)
		{
			AudioDevice->RegisterSoundSubmix(TestSubmix.Get(), true);
		}
		ASSERT_THAT(IsNotNull(TestSubmix.Get()));

		// 3) Create the test sine tone
		SineGenerator = TStrongObjectPtr<UTestSineSynthComponent>(NewObject<UTestSineSynthComponent>());
		SineGenerator->FrequencyHz = FrequencyHz;
		SineGenerator->bAutoActivate = false;
		SineGenerator->bIsUISound = true;
		SineGenerator->bAllowSpatialization = false;
		SineGenerator->CreateAudioComponent();
		SineGenerator->SetSubmixSend(TestSubmix.Get(), TargetSendLevel);

		// Start audio
		SineGenerator->Start();

		// 4) Add an interference tone that sends stuff directly to the final submix just to make sure we aren't defaulting to using main
		InterferenceTone = TStrongObjectPtr<UTestSineSynthComponent>(NewObject<UTestSineSynthComponent>());
		InterferenceTone->FrequencyHz = FrequencyHz * 0.707f;
		InterferenceTone->bAutoActivate = false;
		InterferenceTone->bIsUISound = true;
		InterferenceTone->bAllowSpatialization = false;
		InterferenceTone->CreateAudioComponent();

		if (AudioDevice)
		{
			InterferenceTone->SetSubmixSend(&AudioDevice->GetMainSubmixObject(), TargetSendLevel);
		}

		// Start audio
		InterferenceTone->Start();

		// 4) Create the VoiceChat simulation tester
		VoiceChatSim = MakeUnique<FCQTestPoller>(AudioDevice, TestSubmix.Get(), FrameCount, ChannelCount, SampleRate);

		// 5) Start the Consumer thread
		VoiceChatSim->ClearSetupValues();
		VoiceChatSim->SetTestSineFrequency(FrequencyHz);
		VoiceChatSim->Start();
	}

	AFTER_EACH()
	{
		VoiceChatSim->Stop();
		VoiceChatSim.Reset();

		if (SineGenerator)
		{
			SineGenerator->Stop();
			SineGenerator = nullptr;
		}

		if (InterferenceTone)
		{
			InterferenceTone->Stop();
			InterferenceTone = nullptr;
		}

		if (AudioDevice)
		{
			AudioDevice->UnregisterSoundSubmix(TestSubmix.Get(), false);
		}

		TestSubmix = nullptr;
	}

	// Wait for the listener to receive a few buffers and show non-zero RMS
	// then listen for a few buffers of data to make sure it is the sine wave we expect
	TEST_METHOD(PlaysSineToSubmixAndIsObservedInBuffer)
	{
		// Wait until we get a few buffers with non-trivial energy
		TestCommandBuilder
			.WaitDelay(FTimespan::FromMilliseconds(1000))

			// Wait until the RMS indicates audible signal present
			.Until(TEXT("Wait for non-zero RMS"),
				[this]()
				{
					FirstTestBuffer = VoiceChatSim->GetBuffersReceived();
					VoiceChatSim->ClearTestValues();
					return VoiceChatSim->GetLatestRMS() > MinDetectableRMSForSignal;
				})

			// Run the test for a number of received buffers
			.Until(TEXT("Wait for submix listener to receive buffers"),
				[this]()
				{
					return VoiceChatSim->GetBuffersReceived() > FirstTestBuffer + 10;
				})

			.Then([this]()
				{
					// Final assert: we observed a valid sine wave for every buffer
					const int32 SuccessfulSineTest = VoiceChatSim->GetSuccessfulTestCount();
					const int32 FailedSineTest = VoiceChatSim->GetFailedTestCount();
					ASSERT_THAT(IsTrue(SuccessfulSineTest > 0 && FailedSineTest == 0));
				});
	}

	// Verify that changing the output format mid-stream via SetOutputFormat() produces
	// valid sine wave data at the new format after the transition
	TEST_METHOD(ChangeOutputFormatWhileRunning)
	{
		static constexpr int32 NewChannelCount = 1;
		static constexpr uint32 NewSampleRate = 24000;

		TestCommandBuilder
			.WaitDelay(FTimespan::FromMilliseconds(1000))

			// Phase 1: Wait for valid signal at the original format (stereo 48kHz)
			.Until(TEXT("Wait for non-zero RMS at original format"),
				[this]()
				{
					FirstTestBuffer = VoiceChatSim->GetBuffersReceived();
					VoiceChatSim->ClearTestValues();
					return VoiceChatSim->GetLatestRMS() > MinDetectableRMSForSignal;
				})

			.Until(TEXT("Validate sine at original format"),
				[this]()
				{
					return VoiceChatSim->GetBuffersReceived() > FirstTestBuffer + 10;
				})

			.Then([this]()
				{
					// Confirm original format works
					const int32 SuccessfulSineTest = VoiceChatSim->GetSuccessfulTestCount();
					const int32 FailedSineTest = VoiceChatSim->GetFailedTestCount();
					ASSERT_THAT(IsTrue(SuccessfulSineTest > 0 && FailedSineTest == 0));
				})

			// Phase 2: Switch to mono 24kHz and validate
			.Then([this]()
				{
					VoiceChatSim->SetOutputFormat(NewChannelCount, NewSampleRate);
					VoiceChatSim->ClearSetupValues();
					VoiceChatSim->ClearTestValues();
				})

			.WaitDelay(FTimespan::FromMilliseconds(500))

			.Until(TEXT("Wait for non-zero RMS at new format"),
				[this]()
				{
					FirstTestBuffer = VoiceChatSim->GetBuffersReceived();
					VoiceChatSim->ClearTestValues();
					return VoiceChatSim->GetLatestRMS() > MinDetectableRMSForSignal;
				})

			.Until(TEXT("Validate sine at new format"),
				[this]()
				{
					return VoiceChatSim->GetBuffersReceived() > FirstTestBuffer + 10;
				})

			.Then([this]()
				{
					// Confirm new format produces valid sine data
					const int32 SuccessfulSineTest = VoiceChatSim->GetSuccessfulTestCount();
					const int32 FailedSineTest = VoiceChatSim->GetFailedTestCount();
					ASSERT_THAT(IsTrue(SuccessfulSineTest > 0 && FailedSineTest == 0));
				});
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS