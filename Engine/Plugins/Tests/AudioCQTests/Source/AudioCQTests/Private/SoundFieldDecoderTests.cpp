// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "CoreMinimal.h"
#include "CQTest.h"
#include "SoundFieldRendering.h"
#include "ISoundfieldFormat.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"

// ============================================================================
// Helpers
// ============================================================================

namespace SoundFieldTestHelpers
{
	// Create a stereo speaker layout (FrontLeft=330deg, FrontRight=30deg)
	static TArray<Audio::FChannelPositionInfo> MakeStereoPositions()
	{
		TArray<Audio::FChannelPositionInfo> Positions;
		Positions.Add({EAudioMixerChannel::FrontLeft, -30.f, 0.f});
		Positions.Add({EAudioMixerChannel::FrontRight, 30.f, 0.f});
		return Positions;
	}

	// Create a quad speaker layout
	static TArray<Audio::FChannelPositionInfo> MakeQuadPositions()
	{
		TArray<Audio::FChannelPositionInfo> Positions;
		Positions.Add({EAudioMixerChannel::FrontLeft, -30.f, 0.f});
		Positions.Add({EAudioMixerChannel::FrontRight, 30.f, 0.f});
		Positions.Add({EAudioMixerChannel::BackLeft, -150.f, 0.f});
		Positions.Add({EAudioMixerChannel::BackRight, 150.f, 0.f});
		return Positions;
	}

	// Create N evenly-spaced horizontal speakers starting at 0 degrees (front), stepping clockwise.
	// Supports up to 12 speakers. Avoids FrontCenter (skipped by decoder) and LowFrequency channels.
	static TArray<Audio::FChannelPositionInfo> MakeCircularPositions(int32 N)
	{
		// Channel enum values that are neither FrontCenter (2) nor LowFrequency (3), in order
		static const EAudioMixerChannel::Type CircularChannels[] = {
			EAudioMixerChannel::FrontLeft,         // index 0
			EAudioMixerChannel::FrontRight,        // index 1
			EAudioMixerChannel::BackLeft,          // index 2
			EAudioMixerChannel::BackRight,         // index 3
			EAudioMixerChannel::FrontLeftOfCenter, // index 4
			EAudioMixerChannel::FrontRightOfCenter,// index 5
			EAudioMixerChannel::BackCenter,        // index 6
			EAudioMixerChannel::SideLeft,          // index 7
			EAudioMixerChannel::SideRight,         // index 8
			EAudioMixerChannel::TopCenter,         // index 9
			EAudioMixerChannel::TopFrontLeft,      // index 10
			EAudioMixerChannel::TopFrontCenter,    // index 11
		};
		static const int32 MaxCircularSpeakers = UE_ARRAY_COUNT(CircularChannels);
		check(N > 0 && N <= MaxCircularSpeakers);

		TArray<Audio::FChannelPositionInfo> Positions;
		Positions.Reserve(N);

		const float StepDeg = 360.f / static_cast<float>(N);
		for (int32 i = 0; i < N; ++i)
		{
			float Azimuth = i * StepDeg;
			// Normalize to (-180, 180]
			if (Azimuth > 180.f)
			{
				Azimuth -= 360.f;
			}
			Positions.Add({CircularChannels[i], Azimuth, 0.f});
		}
		return Positions;
	}

	// Compute RMS energy of a buffer using optimized DSP
	static float ComputeRMS(TArrayView<const float> Buffer)
	{
		if (Buffer.Num() == 0)
		{
			return 0.f;
		}
		float MeanSquared = 0.f;
		Audio::ArrayMeanSquared(Buffer, MeanSquared);
		return FMath::Sqrt(MeanSquared);
	}

	// Check if all samples in a buffer are near zero using optimized DSP
	static bool IsBufferSilent(TArrayView<const float> Buffer, float Threshold = 1e-6f)
	{
		float Magnitude = Audio::ArrayGetMagnitude(Buffer);
		return Magnitude < Threshold;
	}

	// Compute per-channel RMS for interleaved audio by striding through in place
	static TArray<float> ComputePerChannelRMS(TArrayView<const float> Buffer, int32 NumChannels)
	{
		const int32 NumFrames = Buffer.Num() / NumChannels;
		TArray<float> ChannelRMS;
		ChannelRMS.SetNumZeroed(NumChannels);

		for (int32 Ch = 0; Ch < NumChannels; ++Ch)
		{
			float Sum = 0.0;
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				const float Val = Buffer[Frame * NumChannels + Ch];
				Sum += Val * Val;
			}
			ChannelRMS[Ch] = FMath::Sqrt(Sum / NumFrames);
		}
		return ChannelRMS;
	}

	// Fill an FOA buffer with a W-channel-only signal (omnidirectional).
	// W channel = index 0, Y/Z/X = indices 1-3
	static void FillFoaOmni(FAmbisonicsSoundfieldBuffer& Buffer, int32 NumFrames, float Amplitude)
	{
		const int32 NumChannels = 4;
		Buffer.NumChannels = NumChannels;
		Buffer.AudioBuffer.Reset();
		Buffer.AudioBuffer.AddZeroed(NumFrames * NumChannels);
		Buffer.Rotation = FQuat::Identity;
		Buffer.PreviousRotation = FQuat::Identity;

		float* Data = Buffer.AudioBuffer.GetData();
		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			Data[Frame * NumChannels] = Amplitude; // W
		}
	}

	// Encode a sine wave from specific speaker positions with per-channel amplitudes.
	// Uses the engine's encoder to produce correctly-encoded ambisonics buffers
	// that are consistent with the decoder's coordinate conventions.
	static void EncodeFromSpeakers(FSoundFieldEncoder& Encoder, int32 Order, int32 NumFrames, float Frequency, float SampleRate,
		TArray<Audio::FChannelPositionInfo>& Positions, const TArray<float>& ChannelAmplitudes, FAmbisonicsSoundfieldBuffer& OutAmbi)
	{
		const int32 NumChannels = Positions.Num();
		Audio::FAlignedFloatBuffer InputData;
		InputData.AddZeroed(NumFrames * NumChannels);
		float* Data = InputData.GetData();
		const float Omega = UE_TWO_PI * Frequency / SampleRate;

		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			const float Signal = FMath::Sin(Omega * Frame);
			for (int32 Ch = 0; Ch < NumChannels; ++Ch)
			{
				Data[Frame * NumChannels + Ch] = Signal * ChannelAmplitudes[Ch];
			}
		}

		FSoundfieldSpeakerPositionalData EncodePos;
		EncodePos.NumChannels = NumChannels;
		EncodePos.ChannelPositions = &Positions;

		FAmbisonicsSoundfieldSettings Settings;
		Settings.Order = Order;

		Encoder.EncodeAudioDirectlyFromOutputPositions(InputData, EncodePos, Settings, OutAmbi);
	}

	// Encode a left-only stereo signal (FL active, FR silent)
	static void EncodeLeftOnly(FSoundFieldEncoder& Encoder, int32 Order, int32 NumFrames, float Frequency, float SampleRate, FAmbisonicsSoundfieldBuffer& OutAmbi)
	{
		TArray<Audio::FChannelPositionInfo> Positions = MakeStereoPositions();
		TArray<float> Amplitudes = { 1.0f, 0.0f };
		EncodeFromSpeakers(Encoder, Order, NumFrames, Frequency, SampleRate, Positions, Amplitudes, OutAmbi);
	}

	// Encode a right-only stereo signal (FR active, FL silent)
	static void EncodeRightOnly(FSoundFieldEncoder& Encoder, int32 Order, int32 NumFrames, float Frequency, float SampleRate, FAmbisonicsSoundfieldBuffer& OutAmbi)
	{
		TArray<Audio::FChannelPositionInfo> Positions = MakeStereoPositions();
		TArray<float> Amplitudes = { 0.0f, 1.0f };
		EncodeFromSpeakers(Encoder, Order, NumFrames, Frequency, SampleRate, Positions, Amplitudes, OutAmbi);
	}

	// Encode a front-only quad signal (FL+FR active, BL+BR silent)
	static void EncodeFrontOnly(FSoundFieldEncoder& Encoder, int32 Order, int32 NumFrames, float Frequency, float SampleRate, FAmbisonicsSoundfieldBuffer& OutAmbi)
	{
		TArray<Audio::FChannelPositionInfo> Positions = MakeQuadPositions();
		TArray<float> Amplitudes = { 1.0f, 1.0f, 0.0f, 0.0f };
		EncodeFromSpeakers(Encoder, Order, NumFrames, Frequency, SampleRate, Positions, Amplitudes, OutAmbi);
	}

	// Encode a rear-only quad signal (BL+BR active, FL+FR silent)
	static void EncodeRearOnly(FSoundFieldEncoder& Encoder, int32 Order, int32 NumFrames, float Frequency, float SampleRate, FAmbisonicsSoundfieldBuffer& OutAmbi)
	{
		TArray<Audio::FChannelPositionInfo> Positions = MakeQuadPositions();
		TArray<float> Amplitudes = { 0.0f, 0.0f, 1.0f, 1.0f };
		EncodeFromSpeakers(Encoder, Order, NumFrames, Frequency, SampleRate, Positions, Amplitudes, OutAmbi);
	}

	// Encode a single mono source at the specified azimuth angle (degrees, 0=front, positive=right).
	// Uses FrontLeft channel type (not FrontCenter) so the encoder does not skip it.
	static void EncodeMonoAt(FSoundFieldEncoder& Encoder, int32 Order, int32 NumFrames, float Frequency, float SampleRate, float AzimuthDeg, FAmbisonicsSoundfieldBuffer& OutAmbi)
	{
		TArray<Audio::FChannelPositionInfo> Positions;
		Positions.Add({EAudioMixerChannel::FrontLeft, AzimuthDeg, 0.f});
		TArray<float> Amplitudes = { 1.0f };
		EncodeFromSpeakers(Encoder, Order, NumFrames, Frequency, SampleRate, Positions, Amplitudes, OutAmbi);
	}
}

// ============================================================================
// FAmbisonicsSoundfieldBuffer Tests
// ============================================================================

TEST_CLASS(SoundFieldBuffer_Tests, "Audio.SoundFieldDecoder.Buffer")
{
	TEST_METHOD(DefaultConstruction_HasZeroChannelsAndIdentityRotation)
	{
		TestCommandBuilder.Do([this]()
		{
			FAmbisonicsSoundfieldBuffer Buffer;
			ASSERT_THAT(AreEqual(Buffer.NumChannels, 0));
			ASSERT_THAT(AreEqual(Buffer.AudioBuffer.Num(), 0));
			ASSERT_THAT(IsTrue(Buffer.Rotation.Equals(FQuat::Identity)));
			ASSERT_THAT(IsTrue(Buffer.PreviousRotation.Equals(FQuat::Identity)));
		});
	}

	TEST_METHOD(Reset_ClearsBufferAndChannelCount)
	{
		TestCommandBuilder.Do([this]()
		{
			FAmbisonicsSoundfieldBuffer Buffer;
			SoundFieldTestHelpers::FillFoaOmni(Buffer, 128, 1.0f);

			ASSERT_THAT(AreEqual(Buffer.NumChannels, 4));
			ASSERT_THAT(AreEqual(Buffer.AudioBuffer.Num(), 512));

			Buffer.Reset();

			ASSERT_THAT(AreEqual(Buffer.NumChannels, 0));
			ASSERT_THAT(AreEqual(Buffer.AudioBuffer.Num(), 0));
		});
	}

	TEST_METHOD(Duplicate_CreatesIndependentCopy)
	{
		TestCommandBuilder.Do([this]()
		{
			FAmbisonicsSoundfieldBuffer Original;
			SoundFieldTestHelpers::FillFoaOmni(Original, 64, 0.5f);
			Original.Rotation = FQuat(FVector::UpVector, PI / 4.0);

			TUniquePtr<ISoundfieldAudioPacket> DuplicatePacket = Original.Duplicate();
			FAmbisonicsSoundfieldBuffer* Duplicate = static_cast<FAmbisonicsSoundfieldBuffer*>(DuplicatePacket.Get());

			ASSERT_THAT(IsNotNull(Duplicate));
			ASSERT_THAT(AreEqual(Duplicate->NumChannels, Original.NumChannels));
			ASSERT_THAT(AreEqual(Duplicate->AudioBuffer.Num(), Original.AudioBuffer.Num()));
			ASSERT_THAT(IsTrue(Duplicate->Rotation.Equals(Original.Rotation)));

			// Verify independence: modify original, check duplicate unchanged
			const float DuplicateFirstSample = Duplicate->AudioBuffer[0];
			Original.AudioBuffer[0] = 999.f;
			ASSERT_THAT(IsNear(Duplicate->AudioBuffer[0], DuplicateFirstSample, 1e-6f));
		});
	}
};

// ============================================================================
// FAmbisonicsSoundfieldSettings Tests
// ============================================================================

TEST_CLASS(SoundFieldSettings_Tests, "Audio.SoundFieldDecoder.Settings")
{
	TEST_METHOD(GetUniqueId_ReturnsOrder)
	{
		TestCommandBuilder.Do([this]()
		{
			FAmbisonicsSoundfieldSettings Settings;
			Settings.Order = 3;
			ASSERT_THAT(AreEqual(Settings.GetUniqueId(), 3u));

			Settings.Order = 1;
			ASSERT_THAT(AreEqual(Settings.GetUniqueId(), 1u));
		});
	}

	TEST_METHOD(Duplicate_CopiesOrder)
	{
		TestCommandBuilder.Do([this]()
		{
			FAmbisonicsSoundfieldSettings Original;
			Original.Order = 5;

			TUniquePtr<ISoundfieldEncodingSettingsProxy> DuplicateProxy = Original.Duplicate();
			FAmbisonicsSoundfieldSettings* Duplicate = static_cast<FAmbisonicsSoundfieldSettings*>(DuplicateProxy.Get());

			ASSERT_THAT(IsNotNull(Duplicate));
			ASSERT_THAT(AreEqual(Duplicate->Order, 5));
		});
	}
};

// ============================================================================
// FSoundFieldDecoder Tests
// ============================================================================

TEST_CLASS(SoundFieldDecoder_Tests, "Audio.SoundFieldDecoder.Decode")
{
	TUniquePtr<FSoundFieldDecoder> Decoder;
	TUniquePtr<FSoundFieldEncoder> Encoder;

	BEFORE_EACH()
	{
		Decoder = MakeUnique<FSoundFieldDecoder>();
		Encoder = MakeUnique<FSoundFieldEncoder>();
	}

	AFTER_EACH()
	{
		Decoder.Reset();
		Encoder.Reset();
	}

	TEST_METHOD(EmptyInput_ProducesSilence)
	{
		TestCommandBuilder.Do([this]()
		{
			FAmbisonicsSoundfieldBuffer Input;
			Input.NumChannels = 0;

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeStereoPositions();
			FSoundfieldSpeakerPositionalData OutputPositions;
			OutputPositions.NumChannels = 2;
			OutputPositions.ChannelPositions = &Positions;

			const int32 NumFrames = 128;
			Audio::FAlignedFloatBuffer OutputData;
			OutputData.AddZeroed(NumFrames * 2);
			// Put non-zero data to verify it gets zeroed
			OutputData[0] = 1.0f;

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(Input, OutputPositions, OutputData);

			ASSERT_THAT(IsTrue(SoundFieldTestHelpers::IsBufferSilent(OutputData)));
		});
	}

	TEST_METHOD(FoaOmniInput_ProducesNonZeroOutputOnAllSpeakers)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer Input;
			SoundFieldTestHelpers::FillFoaOmni(Input, NumFrames, 1.0f);

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeStereoPositions();
			FSoundfieldSpeakerPositionalData OutputPositions;
			OutputPositions.NumChannels = 2;
			OutputPositions.ChannelPositions = &Positions;

			Audio::FAlignedFloatBuffer OutputData;
			OutputData.AddZeroed(NumFrames * 2);

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(Input, OutputPositions, OutputData);

			// An omnidirectional source should produce energy in both speakers
			TArray<float> ChannelRMS = SoundFieldTestHelpers::ComputePerChannelRMS(OutputData, 2);
			ASSERT_THAT(IsTrue(ChannelRMS[0] > 0.01f));
			ASSERT_THAT(IsTrue(ChannelRMS[1] > 0.01f));

			// For an omni source (symmetric W-only), both speakers should have similar energy
			ASSERT_THAT(IsNear(ChannelRMS[0], ChannelRMS[1], 0.05f));
		});
	}

	TEST_METHOD(FoaOmniInput_QuadDecode_ProducesSymmetricOutput)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer Input;
			SoundFieldTestHelpers::FillFoaOmni(Input, NumFrames, 1.0f);

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeQuadPositions();
			FSoundfieldSpeakerPositionalData OutputPositions;
			OutputPositions.NumChannels = 4;
			OutputPositions.ChannelPositions = &Positions;

			Audio::FAlignedFloatBuffer OutputData;
			OutputData.AddZeroed(NumFrames * 4);

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(Input, OutputPositions, OutputData);

			// All 4 speakers should have similar energy for an omnidirectional source
			TArray<float> ChannelRMS = SoundFieldTestHelpers::ComputePerChannelRMS(OutputData, 4);
			for (int32 Ch = 0; Ch < 4; ++Ch)
			{
				ASSERT_THAT(IsTrue(ChannelRMS[Ch] > 0.01f));
			}

			// Front L/R should be similar, Back L/R should be similar
			ASSERT_THAT(IsNear(ChannelRMS[0], ChannelRMS[1], 0.05f));
			ASSERT_THAT(IsNear(ChannelRMS[2], ChannelRMS[3], 0.05f));
		});
	}

	TEST_METHOD(FoaDirectionalFront_QuadDecode_FrontSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			// Encode a front-only source through the engine's encoder so that the
			// ambisonics buffer uses the same coordinate conventions as the decoder.
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer Input;
			SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, 1, NumFrames, 440.f, 48000.f, Input);

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeQuadPositions();
			FSoundfieldSpeakerPositionalData OutputPositions;
			OutputPositions.NumChannels = 4;
			OutputPositions.ChannelPositions = &Positions;

			Audio::FAlignedFloatBuffer OutputData;
			OutputData.AddZeroed(NumFrames * 4);

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(Input, OutputPositions, OutputData);

			TArray<float> ChannelRMS = SoundFieldTestHelpers::ComputePerChannelRMS(OutputData, 4);
			const float FrontEnergy = ChannelRMS[0] + ChannelRMS[1];
			const float BackEnergy = ChannelRMS[2] + ChannelRMS[3];

			// Front speakers should receive more energy for a front-facing source
			ASSERT_THAT(IsTrue(FrontEnergy > BackEnergy));
		});
	}

	TEST_METHOD(FoaDirectionalLeft_StereoDecode_LeftSpeakerLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			// Encode a left-only stereo signal so the ambisonics buffer
			// correctly represents a source at the left speaker position.
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer Input;
			SoundFieldTestHelpers::EncodeLeftOnly(*Encoder, 1, NumFrames, 440.f, 48000.f, Input);

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeStereoPositions();
			FSoundfieldSpeakerPositionalData OutputPositions;
			OutputPositions.NumChannels = 2;
			OutputPositions.ChannelPositions = &Positions;

			Audio::FAlignedFloatBuffer OutputData;
			OutputData.AddZeroed(NumFrames * 2);

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(Input, OutputPositions, OutputData);

			TArray<float> ChannelRMS = SoundFieldTestHelpers::ComputePerChannelRMS(OutputData, 2);

			// Left speaker (index 0) should be louder for a left-side source
			ASSERT_THAT(IsTrue(ChannelRMS[0] > ChannelRMS[1]));
		});
	}

	TEST_METHOD(FoaDirectionalRight_StereoDecode_RightSpeakerLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			// Encode a right-only stereo signal so the ambisonics buffer
			// correctly represents a source at the right speaker position.
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer Input;
			SoundFieldTestHelpers::EncodeRightOnly(*Encoder, 1, NumFrames, 440.f, 48000.f, Input);

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeStereoPositions();
			FSoundfieldSpeakerPositionalData OutputPositions;
			OutputPositions.NumChannels = 2;
			OutputPositions.ChannelPositions = &Positions;

			Audio::FAlignedFloatBuffer OutputData;
			OutputData.AddZeroed(NumFrames * 2);

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(Input, OutputPositions, OutputData);

			TArray<float> ChannelRMS = SoundFieldTestHelpers::ComputePerChannelRMS(OutputData, 2);

			// Right speaker (index 1) should be louder for a right-side source
			ASSERT_THAT(IsTrue(ChannelRMS[1] > ChannelRMS[0]));
		});
	}

	TEST_METHOD(FoaDirectionalRear_QuadDecode_BackSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			// Encode a rear-only source through the engine's encoder.
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer Input;
			SoundFieldTestHelpers::EncodeRearOnly(*Encoder, 1, NumFrames, 440.f, 48000.f, Input);

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeQuadPositions();
			FSoundfieldSpeakerPositionalData OutputPositions;
			OutputPositions.NumChannels = 4;
			OutputPositions.ChannelPositions = &Positions;

			Audio::FAlignedFloatBuffer OutputData;
			OutputData.AddZeroed(NumFrames * 4);

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(Input, OutputPositions, OutputData);

			TArray<float> ChannelRMS = SoundFieldTestHelpers::ComputePerChannelRMS(OutputData, 4);
			const float FrontEnergy = ChannelRMS[0] + ChannelRMS[1];
			const float BackEnergy = ChannelRMS[2] + ChannelRMS[3];

			// Rear speakers should receive more energy for a rear source
			ASSERT_THAT(IsTrue(BackEnergy > FrontEnergy));
		});
	}
};

// ============================================================================
// FOA Rotation Tests
// ============================================================================

TEST_CLASS(SoundFieldRotation_Tests, "Audio.SoundFieldDecoder.Rotation")
{
	TEST_METHOD(FoaRotationInPlace_PreservesTotalEnergy)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 128;
			const int32 NumChannels = 4; // FOA

			Audio::FAlignedFloatBuffer Buffer;
			Buffer.AddZeroed(NumFrames * NumChannels);
			float* Data = Buffer.GetData();

			// Fill with a known FOA signal: W=1, Y=0.5, Z=0.3, X=0.7
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				Data[Frame * 4 + 0] = 1.0f;
				Data[Frame * 4 + 1] = 0.5f;
				Data[Frame * 4 + 2] = 0.3f;
				Data[Frame * 4 + 3] = 0.7f;
			}

			const float EnergyBefore = SoundFieldTestHelpers::ComputeRMS(Buffer);

			// Apply a 90-degree yaw rotation
			FSoundFieldDecoder::FoaRotationInPlace(Buffer, 0.f, 0.f, 90.f);

			const float EnergyAfter = SoundFieldTestHelpers::ComputeRMS(Buffer);

			// Rotation should preserve total energy (within tolerance for floating point)
			ASSERT_THAT(IsNear(EnergyBefore, EnergyAfter, 0.01f));
		});
	}

	TEST_METHOD(FoaRotationInPlace_WChannelPreservedUnderRotation)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 128;
			const int32 NumChannels = 4;

			Audio::FAlignedFloatBuffer Buffer;
			Buffer.AddZeroed(NumFrames * NumChannels);
			float* Data = Buffer.GetData();

			// W-only signal (omnidirectional). Under any rotation, the W channel
			// of a proper FOA rotation matrix should remain unchanged (it's the
			// zeroth-order spherical harmonic which is rotation-invariant).
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				Data[Frame * 4 + 0] = 0.75f; // W
			}

			FSoundFieldDecoder::FoaRotationInPlace(Buffer, 45.f, 30.f, 60.f);

			// W channel should be preserved
			const float* RotatedData = Buffer.GetData();
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				ASSERT_THAT(IsNear(RotatedData[Frame * 4 + 0], 0.75f, 0.01f));
			}
		});
	}

	TEST_METHOD(FoaRotationInPlace_DirectionalSignal_ShiftsYXChannels)
	{
		TestCommandBuilder.Do([this]()
		{
			// A pure X-channel signal (front-facing) rotated 90 degrees yaw
			// should shift energy from X into Y (or vice versa).
			const int32 NumFrames = 128;
			const int32 NumChannels = 4;

			Audio::FAlignedFloatBuffer Buffer;
			Buffer.AddZeroed(NumFrames * NumChannels);
			float* Data = Buffer.GetData();

			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				Data[Frame * 4 + 0] = 0.f;  // W
				Data[Frame * 4 + 1] = 0.f;  // Y
				Data[Frame * 4 + 2] = 0.f;  // Z
				Data[Frame * 4 + 3] = 1.0f; // X (front)
			}

			// Extract X-channel RMS before rotation
			Audio::FAlignedFloatBuffer XChannelBefore;
			XChannelBefore.SetNumUninitialized(NumFrames);
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				XChannelBefore[Frame] = Data[Frame * 4 + 3];
			}
			float XRmsBefore = SoundFieldTestHelpers::ComputeRMS(XChannelBefore);

			FSoundFieldDecoder::FoaRotationInPlace(Buffer, 0.f, 0.f, 90.f);

			// After rotation, the X channel should have less energy (it moved)
			Audio::FAlignedFloatBuffer XChannelAfter;
			XChannelAfter.SetNumUninitialized(NumFrames);
			Audio::FAlignedFloatBuffer YChannelAfter;
			YChannelAfter.SetNumUninitialized(NumFrames);
			const float* RotatedData = Buffer.GetData();
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				YChannelAfter[Frame] = RotatedData[Frame * 4 + 1];
				XChannelAfter[Frame] = RotatedData[Frame * 4 + 3];
			}

			float XRmsAfter = SoundFieldTestHelpers::ComputeRMS(XChannelAfter);
			float YRmsAfter = SoundFieldTestHelpers::ComputeRMS(YChannelAfter);

			// Energy should have transferred from X to Y
			ASSERT_THAT(IsTrue(XRmsAfter < XRmsBefore));
			ASSERT_THAT(IsTrue(YRmsAfter > 0.1f));
		});
	}

	TEST_METHOD(RotateFirstOrderAmbisonicsBed_PreservesEnergy)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 128;
			FAmbisonicsSoundfieldBuffer Input;
			Input.NumChannels = 4;
			Input.AudioBuffer.AddZeroed(NumFrames * 4);
			Input.Rotation = FQuat::Identity;
			Input.PreviousRotation = FQuat::Identity;

			float* InData = Input.AudioBuffer.GetData();
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				InData[Frame * 4 + 0] = 1.0f;  // W
				InData[Frame * 4 + 1] = 0.5f;  // Y
				InData[Frame * 4 + 2] = 0.0f;  // Z
				InData[Frame * 4 + 3] = 0.5f;  // X
			}

			FAmbisonicsSoundfieldBuffer Output;
			Output.NumChannels = 4;
			Output.AudioBuffer.AddZeroed(NumFrames * 4);

			const float EnergyBefore = SoundFieldTestHelpers::ComputeRMS(Input.AudioBuffer);

			// Rotate by 45 degrees yaw (same rotation for prev and dest = no interpolation)
			FQuat Rotation = FQuat(FVector::UpVector, FMath::DegreesToRadians(45.f));
			FSoundFieldDecoder::RotateFirstOrderAmbisonicsBed(Input, Output, Rotation, Rotation);

			const float EnergyAfter = SoundFieldTestHelpers::ComputeRMS(Output.AudioBuffer);

			ASSERT_THAT(IsNear(EnergyBefore, EnergyAfter, 0.02f));
		});
	}

	TEST_METHOD(RotateFirstOrderAmbisonicsBed_IdentityRotation_PreservesSignal)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 64;
			FAmbisonicsSoundfieldBuffer Input;
			Input.NumChannels = 4;
			Input.AudioBuffer.AddZeroed(NumFrames * 4);

			float* InData = Input.AudioBuffer.GetData();
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				InData[Frame * 4 + 0] = 1.0f;
				InData[Frame * 4 + 1] = 0.3f;
				InData[Frame * 4 + 2] = 0.2f;
				InData[Frame * 4 + 3] = 0.8f;
			}

			FAmbisonicsSoundfieldBuffer Output;
			Output.NumChannels = 4;
			Output.AudioBuffer.AddZeroed(NumFrames * 4);

			// Identity rotation should preserve the signal exactly
			FSoundFieldDecoder::RotateFirstOrderAmbisonicsBed(Input, Output, FQuat::Identity, FQuat::Identity);

			const float* OutData = Output.AudioBuffer.GetData();
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				ASSERT_THAT(IsNear(OutData[Frame * 4 + 0], InData[Frame * 4 + 0], 0.01f));
				ASSERT_THAT(IsNear(OutData[Frame * 4 + 1], InData[Frame * 4 + 1], 0.01f));
				ASSERT_THAT(IsNear(OutData[Frame * 4 + 2], InData[Frame * 4 + 2], 0.01f));
				ASSERT_THAT(IsNear(OutData[Frame * 4 + 3], InData[Frame * 4 + 3], 0.01f));
			}
		});
	}

	TEST_METHOD(RotateFirstOrderAmbisonicsBed_DirectionalSource_RotationShiftsDecodeBalance)
	{
		TestCommandBuilder.Do([this]()
		{
			// Encode a front source using the engine's encoder, rotate the bed
			// 180 degrees, then decode. The energy should shift from front to rear.
			const int32 NumFrames = 256;

			// Use encoder to create a properly-encoded front source
			FSoundFieldEncoder LocalEncoder;
			FAmbisonicsSoundfieldBuffer Input;
			SoundFieldTestHelpers::EncodeFrontOnly(LocalEncoder, 1, NumFrames, 440.f, 48000.f, Input);

			// Rotate 180 degrees (should flip front to back)
			FAmbisonicsSoundfieldBuffer Rotated;
			Rotated.NumChannels = 4;
			Rotated.AudioBuffer.AddZeroed(NumFrames * 4);

			FQuat Rotation180 = FQuat(FVector::UpVector, FMath::DegreesToRadians(180.f));
			FSoundFieldDecoder::RotateFirstOrderAmbisonicsBed(Input, Rotated, Rotation180, Rotation180);

			// Decode both original and rotated to quad
			FSoundFieldDecoder DecoderA;
			FSoundFieldDecoder DecoderB;

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeQuadPositions();
			FSoundfieldSpeakerPositionalData OutputPositions;
			OutputPositions.NumChannels = 4;
			OutputPositions.ChannelPositions = &Positions;

			Audio::FAlignedFloatBuffer OriginalOutput;
			OriginalOutput.AddZeroed(NumFrames * 4);
			DecoderA.DecodeAudioDirectlyToDeviceOutputPositions(Input, OutputPositions, OriginalOutput);

			Audio::FAlignedFloatBuffer RotatedOutput;
			RotatedOutput.AddZeroed(NumFrames * 4);
			DecoderB.DecodeAudioDirectlyToDeviceOutputPositions(Rotated, OutputPositions, RotatedOutput);

			TArray<float> OrigRMS = SoundFieldTestHelpers::ComputePerChannelRMS(OriginalOutput, 4);
			TArray<float> RotRMS = SoundFieldTestHelpers::ComputePerChannelRMS(RotatedOutput, 4);

			const float OrigFront = OrigRMS[0] + OrigRMS[1];
			const float OrigBack = OrigRMS[2] + OrigRMS[3];
			const float RotFront = RotRMS[0] + RotRMS[1];
			const float RotBack = RotRMS[2] + RotRMS[3];

			// Original: front > back. After 180 rotation: back > front
			ASSERT_THAT(IsTrue(OrigFront > OrigBack));
			ASSERT_THAT(IsTrue(RotBack > RotFront));
		});
	}
};

// ============================================================================
// FSoundFieldEncoder Tests
// ============================================================================

TEST_CLASS(SoundFieldEncoder_Tests, "Audio.SoundFieldDecoder.Encode")
{
	TUniquePtr<FSoundFieldEncoder> Encoder;

	BEFORE_EACH()
	{
		Encoder = MakeUnique<FSoundFieldEncoder>();
	}

	AFTER_EACH()
	{
		Encoder.Reset();
	}

	TEST_METHOD(EncodeMono_ProducesCorrectChannelCount)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 128;
			Audio::FAlignedFloatBuffer InputData;
			InputData.AddZeroed(NumFrames);

			// Fill with a constant signal
			float* Data = InputData.GetData();
			for (int32 i = 0; i < NumFrames; ++i)
			{
				Data[i] = 0.5f;
			}

			TArray<Audio::FChannelPositionInfo> Positions;
			Positions.Add({EAudioMixerChannel::FrontCenter, 0.f, 0.f});

			FSoundfieldSpeakerPositionalData InputPositions;
			InputPositions.NumChannels = 1;
			InputPositions.ChannelPositions = &Positions;

			FAmbisonicsSoundfieldSettings Settings;
			Settings.Order = 1;

			FAmbisonicsSoundfieldBuffer Output;
			Encoder->EncodeAudioDirectlyFromOutputPositions(InputData, InputPositions, Settings, Output);

			// FOA = (1+1)^2 = 4 channels
			ASSERT_THAT(AreEqual(Output.NumChannels, 4));
			ASSERT_THAT(AreEqual(Output.AudioBuffer.Num(), NumFrames * 4));
		});
	}

	TEST_METHOD(EncodeMono_SecondOrder_ProducesNineChannels)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 64;
			Audio::FAlignedFloatBuffer InputData;
			InputData.AddZeroed(NumFrames);
			float* Data = InputData.GetData();
			for (int32 i = 0; i < NumFrames; ++i)
			{
				Data[i] = 1.0f;
			}

			TArray<Audio::FChannelPositionInfo> Positions;
			Positions.Add({EAudioMixerChannel::FrontCenter, 0.f, 0.f});

			FSoundfieldSpeakerPositionalData InputPositions;
			InputPositions.NumChannels = 1;
			InputPositions.ChannelPositions = &Positions;

			FAmbisonicsSoundfieldSettings Settings;
			Settings.Order = 2;

			FAmbisonicsSoundfieldBuffer Output;
			Encoder->EncodeAudioDirectlyFromOutputPositions(InputData, InputPositions, Settings, Output);

			// Second order = (2+1)^2 = 9 channels
			ASSERT_THAT(AreEqual(Output.NumChannels, 9));
			ASSERT_THAT(AreEqual(Output.AudioBuffer.Num(), NumFrames * 9));
		});
	}

	TEST_METHOD(EncodeMono_ProducesNonZeroOutput)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 128;
			Audio::FAlignedFloatBuffer InputData;
			InputData.AddZeroed(NumFrames);
			float* Data = InputData.GetData();
			for (int32 i = 0; i < NumFrames; ++i)
			{
				Data[i] = 1.0f;
			}

			TArray<Audio::FChannelPositionInfo> Positions;
			Positions.Add({EAudioMixerChannel::FrontCenter, 0.f, 0.f});

			FSoundfieldSpeakerPositionalData InputPositions;
			InputPositions.NumChannels = 1;
			InputPositions.ChannelPositions = &Positions;

			FAmbisonicsSoundfieldSettings Settings;
			Settings.Order = 1;

			FAmbisonicsSoundfieldBuffer Output;
			Encoder->EncodeAudioDirectlyFromOutputPositions(InputData, InputPositions, Settings, Output);

			float RMS = SoundFieldTestHelpers::ComputeRMS(Output.AudioBuffer);
			ASSERT_THAT(IsTrue(RMS > 0.01f));
		});
	}

	TEST_METHOD(EncodeStereo_ProducesNonZeroOutput)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 128;
			const int32 NumInputChannels = 2;
			Audio::FAlignedFloatBuffer InputData;
			InputData.AddZeroed(NumFrames * NumInputChannels);
			float* Data = InputData.GetData();

			// Stereo sine: L=sin, R=cos
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				const float Phase = UE_TWO_PI * 440.f * Frame / 48000.f;
				Data[Frame * 2 + 0] = FMath::Sin(Phase);
				Data[Frame * 2 + 1] = FMath::Cos(Phase);
			}

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeStereoPositions();

			FSoundfieldSpeakerPositionalData InputPositions;
			InputPositions.NumChannels = NumInputChannels;
			InputPositions.ChannelPositions = &Positions;

			FAmbisonicsSoundfieldSettings Settings;
			Settings.Order = 1;

			FAmbisonicsSoundfieldBuffer Output;
			Encoder->EncodeAudioDirectlyFromOutputPositions(InputData, InputPositions, Settings, Output);

			ASSERT_THAT(AreEqual(Output.NumChannels, 4));
			float RMS = SoundFieldTestHelpers::ComputeRMS(Output.AudioBuffer);
			ASSERT_THAT(IsTrue(RMS > 0.01f));
		});
	}

	TEST_METHOD(EncodeStereo_ProducesDirectionalContent)
	{
		TestCommandBuilder.Do([this]()
		{
			// Encode a left-only stereo signal and verify the ambisonics output
			// has both omnidirectional and directional content.
			const int32 NumFrames = 256;
			const int32 NumInputChannels = 2;
			Audio::FAlignedFloatBuffer InputData;
			InputData.AddZeroed(NumFrames * NumInputChannels);
			float* Data = InputData.GetData();

			// Left channel only
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				Data[Frame * 2 + 0] = 1.0f;  // Left
				Data[Frame * 2 + 1] = 0.0f;  // Right (silent)
			}

			TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeStereoPositions();
			FSoundfieldSpeakerPositionalData InputPositions;
			InputPositions.NumChannels = NumInputChannels;
			InputPositions.ChannelPositions = &Positions;

			FAmbisonicsSoundfieldSettings Settings;
			Settings.Order = 1;

			FAmbisonicsSoundfieldBuffer Output;
			Encoder->EncodeAudioDirectlyFromOutputPositions(InputData, InputPositions, Settings, Output);

			// Extract per-ambi-channel energy
			TArray<float> AmbiRMS = SoundFieldTestHelpers::ComputePerChannelRMS(Output.AudioBuffer, 4);

			// W should have energy (omnidirectional component)
			ASSERT_THAT(IsTrue(AmbiRMS[0] > 0.01f));
			// Non-W channels should have directional energy (avoid assuming which
			// specific SH channels carry L/R due to coordinate transform conventions)
			const float DirectionalEnergy = AmbiRMS[1] + AmbiRMS[2] + AmbiRMS[3];
			ASSERT_THAT(IsTrue(DirectionalEnergy > 0.01f));
		});
	}
};

// ============================================================================
// Encode/Decode Roundtrip Tests
// ============================================================================

TEST_CLASS(SoundFieldRoundtrip_Tests, "Audio.SoundFieldDecoder.Roundtrip")
{
	TUniquePtr<FSoundFieldEncoder> Encoder;
	TUniquePtr<FSoundFieldDecoder> Decoder;

	BEFORE_EACH()
	{
		Encoder = MakeUnique<FSoundFieldEncoder>();
		Decoder = MakeUnique<FSoundFieldDecoder>();
	}

	AFTER_EACH()
	{
		Encoder.Reset();
		Decoder.Reset();
	}

	TEST_METHOD(MonoEncodeDecode_PreservesSignalEnergy)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			Audio::FAlignedFloatBuffer InputData;
			InputData.AddZeroed(NumFrames);
			float* Data = InputData.GetData();

			// Mono sine wave
			for (int32 i = 0; i < NumFrames; ++i)
			{
				Data[i] = FMath::Sin(UE_TWO_PI * 440.f * i / 48000.f);
			}

			// Encode: mono -> FOA
			TArray<Audio::FChannelPositionInfo> MonoPositions;
			MonoPositions.Add({EAudioMixerChannel::FrontCenter, 0.f, 0.f});

			FSoundfieldSpeakerPositionalData EncodePositions;
			EncodePositions.NumChannels = 1;
			EncodePositions.ChannelPositions = &MonoPositions;

			FAmbisonicsSoundfieldSettings Settings;
			Settings.Order = 1;

			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			Encoder->EncodeAudioDirectlyFromOutputPositions(InputData, EncodePositions, Settings, AmbiBuffer);

			// Decode: FOA -> stereo
			TArray<Audio::FChannelPositionInfo> StereoPositions = SoundFieldTestHelpers::MakeStereoPositions();
			FSoundfieldSpeakerPositionalData DecodePositions;
			DecodePositions.NumChannels = 2;
			DecodePositions.ChannelPositions = &StereoPositions;

			Audio::FAlignedFloatBuffer DecodedData;
			DecodedData.AddZeroed(NumFrames * 2);

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(AmbiBuffer, DecodePositions, DecodedData);

			// The decoded signal should have meaningful energy
			const float DecodedRMS = SoundFieldTestHelpers::ComputeRMS(DecodedData);
			ASSERT_THAT(IsTrue(DecodedRMS > 0.01f));

			// Both stereo channels should have energy (mono source encoded omni-like)
			TArray<float> ChannelRMS = SoundFieldTestHelpers::ComputePerChannelRMS(DecodedData, 2);
			ASSERT_THAT(IsTrue(ChannelRMS[0] > 0.01f));
			ASSERT_THAT(IsTrue(ChannelRMS[1] > 0.01f));
		});
	}

	TEST_METHOD(StereoLeftOnly_RoundtripsWithLeftBias)
	{
		TestCommandBuilder.Do([this]()
		{
			// Encode a left-only stereo signal through ambisonics and decode
			// back to stereo. The left channel should retain more energy.
			const int32 NumFrames = 256;
			const int32 NumInputChannels = 2;
			Audio::FAlignedFloatBuffer InputData;
			InputData.AddZeroed(NumFrames * NumInputChannels);
			float* Data = InputData.GetData();

			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				Data[Frame * 2 + 0] = FMath::Sin(UE_TWO_PI * 440.f * Frame / 48000.f); // Left
				Data[Frame * 2 + 1] = 0.f; // Right silent
			}

			// Encode
			TArray<Audio::FChannelPositionInfo> StereoPositions = SoundFieldTestHelpers::MakeStereoPositions();
			FSoundfieldSpeakerPositionalData EncodePositions;
			EncodePositions.NumChannels = NumInputChannels;
			EncodePositions.ChannelPositions = &StereoPositions;

			FAmbisonicsSoundfieldSettings Settings;
			Settings.Order = 1;

			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			Encoder->EncodeAudioDirectlyFromOutputPositions(InputData, EncodePositions, Settings, AmbiBuffer);

			// Decode back to stereo
			FSoundfieldSpeakerPositionalData DecodePositions;
			DecodePositions.NumChannels = 2;
			DecodePositions.ChannelPositions = &StereoPositions;

			Audio::FAlignedFloatBuffer DecodedData;
			DecodedData.AddZeroed(NumFrames * 2);

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(AmbiBuffer, DecodePositions, DecodedData);

			TArray<float> ChannelRMS = SoundFieldTestHelpers::ComputePerChannelRMS(DecodedData, 2);

			// Left channel should be louder after roundtrip
			ASSERT_THAT(IsTrue(ChannelRMS[0] > ChannelRMS[1]));
		});
	}

	TEST_METHOD(SilentInput_RoundtripsToSilence)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 128;
			Audio::FAlignedFloatBuffer InputData;
			InputData.AddZeroed(NumFrames); // All zeros

			TArray<Audio::FChannelPositionInfo> MonoPositions;
			MonoPositions.Add({EAudioMixerChannel::FrontCenter, 0.f, 0.f});

			FSoundfieldSpeakerPositionalData EncodePositions;
			EncodePositions.NumChannels = 1;
			EncodePositions.ChannelPositions = &MonoPositions;

			FAmbisonicsSoundfieldSettings Settings;
			Settings.Order = 1;

			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			Encoder->EncodeAudioDirectlyFromOutputPositions(InputData, EncodePositions, Settings, AmbiBuffer);

			// Decode
			TArray<Audio::FChannelPositionInfo> StereoPositions = SoundFieldTestHelpers::MakeStereoPositions();
			FSoundfieldSpeakerPositionalData DecodePositions;
			DecodePositions.NumChannels = 2;
			DecodePositions.ChannelPositions = &StereoPositions;

			Audio::FAlignedFloatBuffer DecodedData;
			DecodedData.AddZeroed(NumFrames * 2);

			Decoder->DecodeAudioDirectlyToDeviceOutputPositions(AmbiBuffer, DecodePositions, DecodedData);

			ASSERT_THAT(IsTrue(SoundFieldTestHelpers::IsBufferSilent(DecodedData)));
		});
	}
};

// ============================================================================
// Higher-Order Ambisonics Tests (Orders 2-5)
// ============================================================================

TEST_CLASS(SoundFieldHigherOrder_Tests, "Audio.SoundFieldDecoder.HigherOrder")
{
	TUniquePtr<FSoundFieldEncoder> Encoder;

	BEFORE_EACH()
	{
		Encoder = MakeUnique<FSoundFieldEncoder>();
	}

	AFTER_EACH()
	{
		Encoder.Reset();
	}

	// Helper: decode an ambisonics buffer to quad and return per-channel RMS
	TArray<float> DecodeToQuadRMS(FAmbisonicsSoundfieldBuffer& AmbiBuffer, int32 NumFrames)
	{
		FSoundFieldDecoder Dec;
		TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeQuadPositions();
		FSoundfieldSpeakerPositionalData DecodePos;
		DecodePos.NumChannels = 4;
		DecodePos.ChannelPositions = &Positions;

		Audio::FAlignedFloatBuffer DecodedData;
		DecodedData.AddZeroed(NumFrames * 4);

		Dec.DecodeAudioDirectlyToDeviceOutputPositions(AmbiBuffer, DecodePos, DecodedData);
		return SoundFieldTestHelpers::ComputePerChannelRMS(DecodedData, 4);
	}

	// Helper: decode to stereo and return per-channel RMS
	TArray<float> DecodeToStereoRMS(FAmbisonicsSoundfieldBuffer& AmbiBuffer, int32 NumFrames)
	{
		FSoundFieldDecoder Dec;
		TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeStereoPositions();
		FSoundfieldSpeakerPositionalData DecodePos;
		DecodePos.NumChannels = 2;
		DecodePos.ChannelPositions = &Positions;

		Audio::FAlignedFloatBuffer DecodedData;
		DecodedData.AddZeroed(NumFrames * 2);

		Dec.DecodeAudioDirectlyToDeviceOutputPositions(AmbiBuffer, DecodePos, DecodedData);
		return SoundFieldTestHelpers::ComputePerChannelRMS(DecodedData, 2);
	}

	// Helper: decode to N evenly-spaced circular speakers and return per-channel RMS.
	// Speakers are at azimuths 0, 360/N, 2*360/N, ... degrees (clockwise from front).
	TArray<float> DecodeToCircularRMS(FAmbisonicsSoundfieldBuffer& AmbiBuffer, int32 NumFrames, int32 NumSpeakers)
	{
		FSoundFieldDecoder Dec;
		TArray<Audio::FChannelPositionInfo> Positions = SoundFieldTestHelpers::MakeCircularPositions(NumSpeakers);
		FSoundfieldSpeakerPositionalData DecodePos;
		DecodePos.NumChannels = NumSpeakers;
		DecodePos.ChannelPositions = &Positions;

		Audio::FAlignedFloatBuffer DecodedData;
		DecodedData.AddZeroed(NumFrames * NumSpeakers);

		Dec.DecodeAudioDirectlyToDeviceOutputPositions(AmbiBuffer, DecodePos, DecodedData);
		return SoundFieldTestHelpers::ComputePerChannelRMS(DecodedData, NumSpeakers);
	}

	// ------------------------------------------------------------------
	// Encode channel count verification for orders 2-5
	// ------------------------------------------------------------------

	TEST_METHOD(EncodeAllOrders_ProducesCorrectChannelCounts)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 128;
			const int32 ExpectedChannels[] = { 4, 9, 16, 25, 36 };

			for (int32 Order = 1; Order <= 5; ++Order)
			{
				FAmbisonicsSoundfieldBuffer AmbiBuffer;
				SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, Order, NumFrames, 440.f, 48000.f, AmbiBuffer);

				const int32 Expected = ExpectedChannels[Order - 1];
				ASSERT_THAT(AreEqual(AmbiBuffer.NumChannels, Expected));
				ASSERT_THAT(AreEqual(AmbiBuffer.AudioBuffer.Num(), NumFrames * Expected));
			}
		});
	}

	// ------------------------------------------------------------------
	// Order 2 (9 channels) - exercises EvenOrderDecodeLoop
	// ------------------------------------------------------------------

	TEST_METHOD(Order2_FrontSource_QuadDecode_FrontSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, 2, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			const float FrontEnergy = RMS[0] + RMS[1];
			const float BackEnergy = RMS[2] + RMS[3];

			ASSERT_THAT(IsTrue(FrontEnergy > BackEnergy));
		});
	}

	TEST_METHOD(Order2_LeftSource_StereoDecode_LeftLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeLeftOnly(*Encoder, 2, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToStereoRMS(AmbiBuffer, NumFrames);

			ASSERT_THAT(IsTrue(RMS[0] > RMS[1]));
		});
	}

	TEST_METHOD(Order2_RearSource_QuadDecode_BackSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeRearOnly(*Encoder, 2, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			const float FrontEnergy = RMS[0] + RMS[1];
			const float BackEnergy = RMS[2] + RMS[3];

			ASSERT_THAT(IsTrue(BackEnergy > FrontEnergy));
		});
	}

	TEST_METHOD(Order2_RoundtripEnergy_IsPreserved)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, 2, NumFrames, 440.f, 48000.f, AmbiBuffer);

			// Encoded ambisonics should have energy
			const float AmbiRMS = SoundFieldTestHelpers::ComputeRMS(AmbiBuffer.AudioBuffer);
			ASSERT_THAT(IsTrue(AmbiRMS > 0.01f));

			// Decoded output should have energy
			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			for (int32 Ch = 0; Ch < 4; ++Ch)
			{
				ASSERT_THAT(IsTrue(RMS[Ch] > 0.001f));
			}
		});
	}

	// ------------------------------------------------------------------
	// Order 3 (16 channels) - exercises OddOrderDecodeLoop
	// ------------------------------------------------------------------

	TEST_METHOD(Order3_FrontSource_QuadDecode_FrontSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, 3, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			const float FrontEnergy = RMS[0] + RMS[1];
			const float BackEnergy = RMS[2] + RMS[3];

			ASSERT_THAT(IsTrue(FrontEnergy > BackEnergy));
		});
	}

	TEST_METHOD(Order3_LeftSource_StereoDecode_LeftLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeLeftOnly(*Encoder, 3, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToStereoRMS(AmbiBuffer, NumFrames);

			ASSERT_THAT(IsTrue(RMS[0] > RMS[1]));
		});
	}

	TEST_METHOD(Order3_RearSource_QuadDecode_BackSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeRearOnly(*Encoder, 3, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			const float FrontEnergy = RMS[0] + RMS[1];
			const float BackEnergy = RMS[2] + RMS[3];

			ASSERT_THAT(IsTrue(BackEnergy > FrontEnergy));
		});
	}

	TEST_METHOD(Order3_RoundtripEnergy_IsPreserved)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, 3, NumFrames, 440.f, 48000.f, AmbiBuffer);

			const float AmbiRMS = SoundFieldTestHelpers::ComputeRMS(AmbiBuffer.AudioBuffer);
			ASSERT_THAT(IsTrue(AmbiRMS > 0.01f));

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			for (int32 Ch = 0; Ch < 4; ++Ch)
			{
				ASSERT_THAT(IsTrue(RMS[Ch] > 0.001f));
			}
		});
	}

	// ------------------------------------------------------------------
	// Order 4 (25 channels) - exercises EvenOrderDecodeLoop
	// ------------------------------------------------------------------

	TEST_METHOD(Order4_FrontSource_QuadDecode_FrontSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, 4, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			const float FrontEnergy = RMS[0] + RMS[1];
			const float BackEnergy = RMS[2] + RMS[3];

			ASSERT_THAT(IsTrue(FrontEnergy > BackEnergy));
		});
	}

	TEST_METHOD(Order4_LeftSource_StereoDecode_LeftLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeLeftOnly(*Encoder, 4, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToStereoRMS(AmbiBuffer, NumFrames);

			ASSERT_THAT(IsTrue(RMS[0] > RMS[1]));
		});
	}

	TEST_METHOD(Order4_RearSource_QuadDecode_BackSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeRearOnly(*Encoder, 4, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			const float FrontEnergy = RMS[0] + RMS[1];
			const float BackEnergy = RMS[2] + RMS[3];

			ASSERT_THAT(IsTrue(BackEnergy > FrontEnergy));
		});
	}

	TEST_METHOD(Order4_RoundtripEnergy_IsPreserved)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, 4, NumFrames, 440.f, 48000.f, AmbiBuffer);

			const float AmbiRMS = SoundFieldTestHelpers::ComputeRMS(AmbiBuffer.AudioBuffer);
			ASSERT_THAT(IsTrue(AmbiRMS > 0.01f));

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			for (int32 Ch = 0; Ch < 4; ++Ch)
			{
				ASSERT_THAT(IsTrue(RMS[Ch] > 0.001f));
			}
		});
	}

	// ------------------------------------------------------------------
	// Order 5 (36 channels) - exercises OddOrderDecodeLoop
	// ------------------------------------------------------------------

	TEST_METHOD(Order5_FrontSource_QuadDecode_FrontSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, 5, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			const float FrontEnergy = RMS[0] + RMS[1];
			const float BackEnergy = RMS[2] + RMS[3];

			ASSERT_THAT(IsTrue(FrontEnergy > BackEnergy));
		});
	}

	TEST_METHOD(Order5_LeftSource_StereoDecode_LeftLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeLeftOnly(*Encoder, 5, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToStereoRMS(AmbiBuffer, NumFrames);

			ASSERT_THAT(IsTrue(RMS[0] > RMS[1]));
		});
	}

	TEST_METHOD(Order5_RearSource_QuadDecode_BackSpeakersLouder)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeRearOnly(*Encoder, 5, NumFrames, 440.f, 48000.f, AmbiBuffer);

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			const float FrontEnergy = RMS[0] + RMS[1];
			const float BackEnergy = RMS[2] + RMS[3];

			ASSERT_THAT(IsTrue(BackEnergy > FrontEnergy));
		});
	}

	TEST_METHOD(Order5_RoundtripEnergy_IsPreserved)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;
			FAmbisonicsSoundfieldBuffer AmbiBuffer;
			SoundFieldTestHelpers::EncodeFrontOnly(*Encoder, 5, NumFrames, 440.f, 48000.f, AmbiBuffer);

			const float AmbiRMS = SoundFieldTestHelpers::ComputeRMS(AmbiBuffer.AudioBuffer);
			ASSERT_THAT(IsTrue(AmbiRMS > 0.01f));

			TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);
			for (int32 Ch = 0; Ch < 4; ++Ch)
			{
				ASSERT_THAT(IsTrue(RMS[Ch] > 0.001f));
			}
		});
	}

	// ------------------------------------------------------------------
	// Silence roundtrip at each order
	// ------------------------------------------------------------------

	TEST_METHOD(AllOrders_SilentInput_RoundtripsToSilence)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 128;

			for (int32 Order = 1; Order <= 5; ++Order)
			{
				Audio::FAlignedFloatBuffer SilentInput;
				SilentInput.AddZeroed(NumFrames);

				TArray<Audio::FChannelPositionInfo> MonoPos;
				MonoPos.Add({EAudioMixerChannel::FrontCenter, 0.f, 0.f});

				FSoundfieldSpeakerPositionalData EncodePos;
				EncodePos.NumChannels = 1;
				EncodePos.ChannelPositions = &MonoPos;

				FAmbisonicsSoundfieldSettings Settings;
				Settings.Order = Order;

				FAmbisonicsSoundfieldBuffer AmbiBuffer;
				Encoder->EncodeAudioDirectlyFromOutputPositions(SilentInput, EncodePos, Settings, AmbiBuffer);

				// Ambisonics buffer should be silent
				ASSERT_THAT(IsTrue(SoundFieldTestHelpers::IsBufferSilent(AmbiBuffer.AudioBuffer)));

				// Decoded output should also be silent
				FSoundFieldDecoder Dec;
				TArray<Audio::FChannelPositionInfo> StereoPos = SoundFieldTestHelpers::MakeStereoPositions();
				FSoundfieldSpeakerPositionalData DecodePos;
				DecodePos.NumChannels = 2;
				DecodePos.ChannelPositions = &StereoPos;

				Audio::FAlignedFloatBuffer DecodedData;
				DecodedData.AddZeroed(NumFrames * 2);

				Dec.DecodeAudioDirectlyToDeviceOutputPositions(AmbiBuffer, DecodePos, DecodedData);
				ASSERT_THAT(IsTrue(SoundFieldTestHelpers::IsBufferSilent(DecodedData)));
			}
		});
	}

	// ------------------------------------------------------------------
	// Spatial resolution: higher orders should produce greater
	// front/back contrast for a front-facing source
	// ------------------------------------------------------------------

	TEST_METHOD(HigherOrders_ProduceGreaterSpatialContrast)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;

			// Encode a single mono source at 0° (front) at each order, decode to
			// a 12-speaker circular array, and compare the gain at index 0 (0°, on-axis)
			// vs index 1 (30°, the nearest adjacent speaker).
			//
			// UE uses ACN/N3D spherical harmonic normalization.  With that convention
			// the on-axis decoded amplitude equals ||E||² = 4, 9, 26.5, 35.5, 46.5 for
			// orders 1-5 (strictly increasing).  At 30° the cos(3θ) term vanishes
			// (cos 90°=0) and the cos(4θ), cos(5θ) terms subtract, so the adjacent
			// amplitude grows more slowly: ≈3.60, 6.72, 13.29, 13.50, 11.04.  The
			// on-axis/adjacent ratio is therefore strictly monotone increasing:
			//   order 1: ≈1.11, order 2: ≈1.34, order 3: ≈1.99,
			//   order 4: ≈2.63, order 5: ≈4.21
			//
			// 12 speakers at 30° intervals satisfies the 2D Nyquist criterion
			// (2N+1 speakers needed for order N) for all orders 1-5.
			static const int32 NumSpeakers = 12;
			float PreviousRatio = 0.f;

			for (int32 Order = 1; Order <= 5; ++Order)
			{
				FAmbisonicsSoundfieldBuffer AmbiBuffer;
				SoundFieldTestHelpers::EncodeMonoAt(*Encoder, Order, NumFrames, 440.f, 48000.f, 0.f, AmbiBuffer);

				TArray<float> RMS = DecodeToCircularRMS(AmbiBuffer, NumFrames, NumSpeakers);
				const float FrontEnergy    = RMS[0];  // speaker at   0° (on-axis with source)
				const float AdjacentEnergy = RMS[1];  // speaker at  30° (nearest neighbour)

				// On-axis speaker should always receive more energy than the adjacent one
				ASSERT_THAT(IsTrue(FrontEnergy > AdjacentEnergy));

				// Ratio on-axis/adjacent is strictly increasing with order (see comment above)
				const float Ratio = (AdjacentEnergy > 1e-8f) ? (FrontEnergy / AdjacentEnergy) : FrontEnergy;

				// Each successive order should produce at least as much contrast
				if (Order > 1)
				{
					ASSERT_THAT(IsTrue(Ratio >= PreviousRatio * 0.95f));
				}

				PreviousRatio = Ratio;
			}
		});
	}

	// ------------------------------------------------------------------
	// Left/right separation: higher orders should give sharper L/R split
	// ------------------------------------------------------------------

	TEST_METHOD(HigherOrders_ProduceGreaterLeftRightSeparation)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;

			// Encode a single mono source at -90° (left side) at each order, decode
			// to a 12-speaker circular array, and compare the gain at index 9 (-90°,
			// on-axis) vs index 10 (-60°, the nearest adjacent speaker, 30° away).
			//
			// By the rotational symmetry of the SH response the on-axis/adjacent
			// ratio at 30° separation is identical to the front/back test above:
			//   order 1: ≈1.11, order 2: ≈1.34, order 3: ≈1.99,
			//   order 4: ≈2.63, order 5: ≈4.21  — strictly monotone increasing.
			static const int32 NumSpeakers = 12;
			float PreviousRatio = 0.f;

			for (int32 Order = 1; Order <= 5; ++Order)
			{
				FAmbisonicsSoundfieldBuffer AmbiBuffer;
				SoundFieldTestHelpers::EncodeMonoAt(*Encoder, Order, NumFrames, 440.f, 48000.f, -90.f, AmbiBuffer);

				TArray<float> RMS = DecodeToCircularRMS(AmbiBuffer, NumFrames, NumSpeakers);
				const float LeftEnergy     = RMS[9];   // speaker at -90° (on-axis with source)
				const float AdjacentEnergy = RMS[10];  // speaker at -60° (nearest neighbour, 30° away)

				// On-axis speaker should always receive more energy than the adjacent one
				ASSERT_THAT(IsTrue(LeftEnergy > AdjacentEnergy));

				// Ratio on-axis/adjacent is strictly increasing with order (see comment above)
				const float Ratio = (AdjacentEnergy > 1e-8f) ? (LeftEnergy / AdjacentEnergy) : LeftEnergy;

				if (Order > 1)
				{
					ASSERT_THAT(IsTrue(Ratio >= PreviousRatio * 0.95f));
				}

				PreviousRatio = Ratio;
			}
		});
	}

	// ------------------------------------------------------------------
	// Centered stereo encode/decode at all orders: L/R symmetric output
	// ------------------------------------------------------------------

	TEST_METHOD(AllOrders_OmniSource_QuadDecode_SymmetricOutput)
	{
		TestCommandBuilder.Do([this]()
		{
			const int32 NumFrames = 256;

			for (int32 Order = 1; Order <= 5; ++Order)
			{
				// Encode identical L+R from symmetric stereo positions
				// to produce a centered field
				const int32 NumInputChannels = 2;
				Audio::FAlignedFloatBuffer StereoInput;
				StereoInput.AddZeroed(NumFrames * NumInputChannels);
				float* StereoData = StereoInput.GetData();
				const float Omega = UE_TWO_PI * 440.f / 48000.f;
				for (int32 Frame = 0; Frame < NumFrames; ++Frame)
				{
					const float Sig = FMath::Sin(Omega * Frame);
					StereoData[Frame * 2 + 0] = Sig; // Left
					StereoData[Frame * 2 + 1] = Sig; // Right (same signal = centered)
				}

				TArray<Audio::FChannelPositionInfo> StereoPos = SoundFieldTestHelpers::MakeStereoPositions();
				FSoundfieldSpeakerPositionalData EncodePos;
				EncodePos.NumChannels = NumInputChannels;
				EncodePos.ChannelPositions = &StereoPos;

				FAmbisonicsSoundfieldSettings Settings;
				Settings.Order = Order;

				FAmbisonicsSoundfieldBuffer AmbiBuffer;
				Encoder->EncodeAudioDirectlyFromOutputPositions(StereoInput, EncodePos, Settings, AmbiBuffer);

				// Decode to quad
				TArray<float> RMS = DecodeToQuadRMS(AmbiBuffer, NumFrames);

				// All speakers should have energy
				for (int32 Ch = 0; Ch < 4; ++Ch)
				{
					ASSERT_THAT(IsTrue(RMS[Ch] > 0.001f));
				}

				// Front L/R should be symmetric (centered stereo guarantees this)
				ASSERT_THAT(IsNear(RMS[0], RMS[1], 0.05f));
			}
		});
	}
};

// ============================================================================
// API / Format Name Tests
// ============================================================================

TEST_CLASS(SoundFieldAPI_Tests, "Audio.SoundFieldDecoder.API")
{
	TEST_METHOD(GetUnrealAmbisonicsFormatName_ReturnsExpectedName)
	{
		TestCommandBuilder.Do([this]()
		{
			FName FormatName = GetUnrealAmbisonicsFormatName();
			ASSERT_THAT(AreEqual(FormatName, FName("Unreal Ambisonics")));
		});
	}

	TEST_METHOD(GetAmbisonicsSourceDefaultSettings_ReturnsFirstOrder)
	{
		TestCommandBuilder.Do([this]()
		{
			ISoundfieldEncodingSettingsProxy& DefaultSettings = GetAmbisonicsSourceDefaultSettings();
			// Default should be first order (UniqueId == 1)
			ASSERT_THAT(AreEqual(DefaultSettings.GetUniqueId(), 1u));
		});
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS
