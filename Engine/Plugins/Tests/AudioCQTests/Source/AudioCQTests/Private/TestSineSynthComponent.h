// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SynthComponent.h"
#include "DSP/SinOsc.h" 
#include "TestSineSynthComponent.generated.h"

UCLASS()
class UTestSineSynthComponent : public USynthComponent
{
    GENERATED_BODY()

public:
    // Frequency Hz
    UPROPERTY(EditAnywhere, Category = "Test|Audio")
    float FrequencyHz = 440.0f;

    // Amplitude 0..1
    UPROPERTY(EditAnywhere, Category = "Test|Audio")
    float Amplitude = 0.25f;

protected:
    // Called by USynthComponent::Initialize
    virtual bool Init(int32& SampleRate) override
    {
        // Sample rate chosen by USynthComponent will be provided here
        Osc.Init((float)SampleRate, FrequencyHz);
		NumChannels = 1;
        return true;
    }

    // Called per render block
    virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override
    {
        // NumSamples accounts for number of channels
        // Generate mono then upmix to component NumChannels
        const int32 NumFrames = NumSamples / NumChannels;

        for (int32 Frame = 0; Frame < NumFrames; ++Frame)
        {
            // FSineOsc::ProcessAudio() returns one sample at osc amplitude
            const float Sample = Osc.ProcessAudio() * Amplitude;
            for (int32 Ch = 0; Ch < NumChannels; ++Ch)
            {
                OutAudio[Frame * NumChannels + Ch] = Sample;
            }
        }
        return NumSamples;
    }

private:
    Audio::FSineOsc Osc;
};